// Copyright Epic Games, Inc. All Rights Reserved.

#include "MDLUSDShadeMaterialTranslator.h"
#include "Engine/Level.h"

#if USE_USD_SDK && WITH_EDITOR

#include "MDLUSDLog.h"
#include "USDAssetCache.h"
#include "USDAssetImportData.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDMemory.h"
#include "USDShadeConversion.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"

#include "Engine/Texture.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceConstant.h"
#include "MaterialShared.h"
#include "MDLImporterOptions.h"
#include "MDLMaterialImporter.h"
#include "Misc/Paths.h"
#include "UObject/StrongObjectPtr.h"

#include "USDIncludesStart.h"
#include "pxr/usd/sdf/assetPath.h"
#include "pxr/usd/usdShade/material.h"
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "MDLUSDShadeMaterialTranslator"

namespace UE::MDLShadeTranslator::Private
{
	void NotifyIfMaterialNeedsVirtualTextures(UMaterialInterface* MaterialInterface)
	{
		if (UMaterial* Material = Cast<UMaterial>(MaterialInterface))
		{
			TArray<UTexture*> UsedTextures;
			Material->GetUsedTextures(UsedTextures);

			for (UTexture* UsedTexture : UsedTextures)
			{
				UsdUtils::NotifyIfVirtualTexturesNeeded(UsedTexture);
			}
		}
		else if (UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(MaterialInterface))
		{
			for (const FTextureParameterValue& TextureValue : MaterialInstance->TextureParameterValues)
			{
				if (UTexture* Texture = TextureValue.ParameterValue)
				{
					UsdUtils::NotifyIfVirtualTexturesNeeded(Texture);
				}
			}
		}
	}
}	 // namespace UE::MDLShadeTranslator::Private

void FMdlUsdShadeMaterialTranslator::CreateAssets()
{
	// MDL USD Schema:
	//   info:mdl:sourceAsset -> Path to the MDL file
	//   info:mdl:sourceAsset:subIdentifier -> Name of the material in the MDL file
	//   inputs -> material parameters

	if (Context->RenderContext != UnrealIdentifiers::MdlRenderContext)
	{
		Super::CreateAssets();
		return;
	}

	if (Context->bTranslateOnlyUsedMaterials && Context->UsdInfoCache)
	{
		if (!Context->UsdInfoCache->IsMaterialUsed(PrimPath))
		{
			UE_LOGF(
				LogUsdMdl,
				Verbose,
				"Skipping creating assets for material prim '%ls' as it is not currently bound by any prim.",
				*PrimPath.GetString()
			);
			return;
		}
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdShadeMaterial ShadeMaterial(GetPrim());

	if (!ShadeMaterial)
	{
		return;
	}

	const static pxr::TfToken MdlToken = UnrealToUsd::ConvertToken(*UnrealIdentifiers::MdlRenderContext.ToString()).Get();

	pxr::UsdShadeShader SurfaceShader = ShadeMaterial.ComputeSurfaceSource(MdlToken);

	if (!SurfaceShader)
	{
		Super::CreateAssets();
		return;
	}

	const FString MdlRootPath = Context->Stage.GetRootLayer().GetRealPath();

	pxr::SdfAssetPath SurfaceSourceAssetPath;
	SurfaceShader.GetSourceAsset(&SurfaceSourceAssetPath, MdlToken);

	if (SurfaceSourceAssetPath.GetAssetPath().empty())
	{
		// Old Mdl Schema
		const pxr::TfToken ModuleToken("module");
		const pxr::UsdAttribute& MDLModule = SurfaceShader.GetPrim().GetAttribute(ModuleToken);

		if (MDLModule.GetTypeName().GetAsToken() == pxr::SdfValueTypeNames->Asset)
		{
			SurfaceSourceAssetPath = UsdUtils::GetUsdValue<pxr::SdfAssetPath>(MDLModule, Context->Time);
		}
	}

	const FString MdlOrigAssetPath = UsdToUnreal::ConvertString(SurfaceSourceAssetPath.GetAssetPath());
	const FString MdlAbsoluteAssetPath = UsdToUnreal::ConvertString(SurfaceSourceAssetPath.GetResolvedPath());

	const FString MdlModuleName = [&MdlAbsoluteAssetPath, &MdlRootPath]()
	{
		FString ModuleRelativePath = MdlAbsoluteAssetPath;
		FPaths::MakePathRelativeTo(ModuleRelativePath, *MdlRootPath);

		FString ModuleName = UE::Mdl::Util::ConvertFilePathToModuleName(*ModuleRelativePath);
		return ModuleName;
	}();

	if (!MdlModuleName.IsEmpty() && Context->UsdAssetCache)
	{
		pxr::TfToken MdlDefinitionToken;
		SurfaceShader.GetSourceAssetSubIdentifier(&MdlDefinitionToken, MdlToken);

		const FString MdlDefinitionName = UsdToUnreal::ConvertToken(MdlDefinitionToken);

		const FString MdlFullName = MdlModuleName + TEXT("::") + MdlDefinitionName;
		const FString MdlFullInstanceName = MdlFullName + TEXT("_Instance");
		const FString HashPrefix = UsdUtils::GetAssetHashPrefix(GetPrim(), Context->bShareAssetsForIdenticalPrims);
		const FString MaterialHash = HashPrefix + MdlFullName;
		const FString MdlSearchPath = FPaths::GetPath(Context->Stage.GetRootLayer().GetRealPath());

		TSet<UObject*> Dependencies;

		FScopedUnrealAllocs UEAllocs;

		// Create reference material
		//
		// The FMdlMaterialImporter will (strangely) take the provided UPackage "ParentPackage" and instead just create
		// another package, using "ParentPackage.GetName() / MdlMaterialName" as it's path... This is not ideal especially
		// since it will lead to asset collisions if we're not reusing assets via hash. Here we just put everything
		// on the transient package and rename the asset into the target destination package instead, like we do for MaterialX
		//
		// We have to do all of that inside the lambda though because we want to reuse that reference material if we have one
		// cached already
		bool bCreatedMaterial = false;
		UMaterialInterface* MdlMaterial = Context->UsdAssetCache->GetOrCreateCustomCachedAsset<UMaterialInterface>(
			MaterialHash,
			MdlFullName,
			Context->ObjectFlags,
			[&MdlSearchPath, &MdlModuleName, &MdlDefinitionName, &MdlOrigAssetPath](UPackage* Outer, FName SanitizedName, EObjectFlags FlagsToUse)
				-> UMaterialInterface*
			{
				// Add the USD root as a search path for MDL
				FMdlMaterialImporter::FScopedSearchPath UsdDirMdlSearchPath(MdlSearchPath);

				TStrongObjectPtr<UMDLImporterOptions> ImportOptions(NewObject<UMDLImporterOptions>());

				UPackage* ParentPackage = GetTransientPackage();
				UMaterialInterface* ReferenceMaterial = FMdlMaterialImporter::ImportMaterialFromModule(
					ParentPackage,
					FlagsToUse,
					MdlModuleName,
					MdlDefinitionName,
					*ImportOptions.Get()
				);
				if (!ReferenceMaterial)
				{
					FString FullPath = FPaths::Combine(MdlSearchPath, MdlOrigAssetPath);
					FPaths::NormalizeFilename(FullPath);
					UE_LOGF(LogUsdMdl, Warning, "Failed to load MDL material from file '%ls'. Does the file exist?", *FullPath);
					return nullptr;
				}

				// Rename the UMaterialInterface into the target UPackage the asset cache created for us.
				// SanitizedName will already match it.
				const bool bRenamed = ReferenceMaterial->Rename(*SanitizedName.ToString(), Outer, REN_NonTransactional | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
				ensure(bRenamed);

				// Let's not trust the flags the MDLImporter used and just use our own instead
				ReferenceMaterial->ClearFlags(ReferenceMaterial->GetFlags());
				ReferenceMaterial->SetFlags(FlagsToUse);

				return ReferenceMaterial;
			},
			&bCreatedMaterial
		);
		if (bCreatedMaterial && MdlMaterial)
		{
			UE::MDLShadeTranslator::Private::NotifyIfMaterialNeedsVirtualTextures(MdlMaterial);
			Dependencies.Append(IUsdClassesModule::GetAssetDependencies(MdlMaterial));
		}
		else if (!MdlMaterial)
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT("UsdMdlConversionFailed", "Failed to create MDL material for prim {0}."),
				FText::FromString(PrimPath.GetString())
			));
		}

		// Create material instance
		bool bCreatedInstance = false;
		const FString MaterialInstanceHash = HashPrefix + MdlFullInstanceName;
		UMaterialInstanceConstant* MdlMaterialInstance = Context->UsdAssetCache->GetOrCreateCachedAsset<UMaterialInstanceConstant>(
			MaterialInstanceHash,
			MdlFullInstanceName,
			Context->ObjectFlags,
			&bCreatedInstance
		);
		if (bCreatedInstance && MdlMaterialInstance)
		{
			MdlMaterialInstance->SetParentEditorOnly(MdlMaterial);

			UsdToUnreal::ConvertShadeInputsToParameters(
				ShadeMaterial,
				*MdlMaterialInstance,
				Context->UsdAssetCache.Get(),
				*Context->RenderContext.ToString(),
				Context->bShareAssetsForIdenticalPrims
			);

			// We can't blindly recreate all component render states when a level is being added, because we may end up first creating
			// render states for some components, and UWorld::AddToWorld calls FScene::AddPrimitive which expects the component to not have
			// primitives yet
			FMaterialUpdateContext::EOptions::Type Options = FMaterialUpdateContext::EOptions::Default;
			if (Context->Level && Context->Level->bIsAssociatingLevel)
			{
				Options = (FMaterialUpdateContext::EOptions::Type)(Options & ~FMaterialUpdateContext::EOptions::RecreateRenderStates);
			}

			FMaterialUpdateContext UpdateContext(Options, GMaxRHIShaderPlatform);
			UpdateContext.AddMaterialInstance(MdlMaterialInstance);
			MdlMaterialInstance->PreEditChange(nullptr);
			MdlMaterialInstance->PostEditChange();

			UE::MDLShadeTranslator::Private::NotifyIfMaterialNeedsVirtualTextures(MdlMaterialInstance);
			Dependencies.Append(IUsdClassesModule::GetAssetDependencies(MdlMaterialInstance));
		}

		// If we created a material here, we could have created some textures. Let's cache those too
		// so that PostImportMaterial can take care of them
		for (UObject* Object : Dependencies)
		{
			if (UTexture* Texture = Cast<UTexture>(Object))
			{
				// We have to watch out because it seems like FMdlMaterialImporter can generate some materials using
				// default engine textures like FlatNormal. We don't want to cache those.
				// Note that we can only test for that by checking whether the asset has been saved or not because the
				// MDL importer (unfortunately) can't use our asset cache with its previously created materials and textures...
				// This means that every time an MDL has to create a new texture it will re-read the image from disk again and create
				// a new (as of yet unsaved) asset. This means if the asset is saved it must have been an engine texture.
				// Also Note: I think the MDLImporter has a bug here where it will put those textures in paths like
				// "/Engine/TextureName"... We can take them anyway though.
				UPackage* TexturePackage = Texture->GetOutermost();
				if (TexturePackage && TexturePackage->GetFileSize() > 0)
				{
					continue;
				}

				// Note how we add MaterialHash to the texture hash: This because unfortunately there's no way to get
				// the MDLImporter to easily use our cached textures (it does have a ResourcesDir on UMDLImporterOptions but
				// those seem to be file directory paths instead?). This means each MDL material must be allowed to create and
				// generate new textures for itself, which we must guarantee don't collide with anything inside the asset cache.
				const FString FilePath = Texture->AssetImportData ? Texture->AssetImportData->GetFirstFilename() : TEXT("");
				const FString PrefixedTextureHash = HashPrefix + MaterialHash
													+ UsdUtils::GetTextureHash(
														FilePath,
														Texture->SRGB,
														Texture->CompressionSettings,
														Texture->GetTextureAddressX(),
														Texture->GetTextureAddressY()
													);

				bool bCreatedTexture = false;
				UObject* PackagedTexture = Context->UsdAssetCache->GetOrCreateCustomCachedAsset(
					PrefixedTextureHash,
					Texture->GetClass(),
					Texture->GetName(),
					Texture->GetFlags(),
					[Texture](UPackage* Outer, FName SanitizedName, EObjectFlags FlagsToUse) -> UObject*
					{
						// Rename the asset into the target UPackage the asset cache created for us.
						// SanitizedName will already match it.
						const bool bRenamed = Texture->Rename(*SanitizedName.ToString(), Outer, REN_NonTransactional | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
						ensure(bRenamed);

						// Let's not trust the flags the MDLImporter used and just use our own instead
						Texture->ClearFlags(Texture->GetFlags());
						Texture->SetFlags(FlagsToUse);

						return Texture;
					},
					&bCreatedTexture
				);
				ensure(PackagedTexture && bCreatedTexture);
			}
		}

		PostImportMaterial(MaterialHash, MdlMaterial);
		PostImportMaterial(MaterialInstanceHash, MdlMaterialInstance);
	}
	else
	{
		Super::CreateAssets();
	}
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK && WITH_EDITOR
