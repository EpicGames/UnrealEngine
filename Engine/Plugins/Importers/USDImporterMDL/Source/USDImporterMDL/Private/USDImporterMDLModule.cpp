// Copyright Epic Games, Inc. All Rights Reserved.

#include "MDLUSDLog.h"
#include "MDLUSDShadeMaterialTranslator.h"
#include "Objects/USDSchemaTranslator.h"
#include "USDMaterialUtils.h"
#include "USDMemory.h"

#include "MDLImporterModule.h"

DEFINE_LOG_CATEGORY(LogUsdMdl);

class FUsdImporterMdlModule : public IModuleInterface
{
private:
	virtual void StartupModule() override
	{
#if USE_USD_SDK && WITH_EDITOR
#ifdef USE_MDLSDK
		if (GIsEditor)
		{
			if (IMDLImporterModule* MDLImporterModule = FModuleManager::Get().LoadModulePtr<IMDLImporterModule>(TEXT("MDLImporter")))
			{
				UsdUnreal::MaterialUtils::RegisterRenderContext(UnrealIdentifiers::MdlRenderContext);

				FUsdSchemaTranslatorRegistry& Registry = FUsdSchemaTranslatorRegistry::Get();
				TranslatorHandle = Registry.Register<FMdlUsdShadeMaterialTranslator>(TEXT("UsdShadeMaterial"));
			}
		}
#else
		UE_LOGF(LogUsdMdl, Log, "Not registering the MDL schema translator as the MDL SDK is not available");
#endif	  // USE_MDL_SDK
#endif	  // WITH_EDITOR
	}

	virtual void ShutdownModule() override
	{
		FUsdSchemaTranslatorRegistry& Registry = FUsdSchemaTranslatorRegistry::Get();
		Registry.Unregister(TranslatorHandle);
	}

private:
	FRegisteredSchemaTranslatorHandle TranslatorHandle;
};

IMPLEMENT_MODULE_USD(FUsdImporterMdlModule, USDImporterMDL);
