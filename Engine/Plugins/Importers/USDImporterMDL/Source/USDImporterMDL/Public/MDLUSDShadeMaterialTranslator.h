// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDShadeMaterialTranslator.h"

#include "Custom/MaterialXUSDShadeMaterialTranslator.h"

#if USE_USD_SDK && WITH_EDITOR

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

class FMdlUsdShadeMaterialTranslator : public FMaterialXUsdShadeMaterialTranslator
{
	using Super = FMaterialXUsdShadeMaterialTranslator;

public:
	using FMaterialXUsdShadeMaterialTranslator::FMaterialXUsdShadeMaterialTranslator;

	virtual void CreateAssets() override;
};

#endif	  // #if USE_USD_SDK && WITH_EDITOR
