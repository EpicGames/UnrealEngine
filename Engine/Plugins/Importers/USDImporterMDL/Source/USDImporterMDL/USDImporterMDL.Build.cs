// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;
using UnrealBuildTool;
using UnrealBuildTool.Rules;

public class USDImporterMDL : ModuleRules
{
	public USDImporterMDL(ReadOnlyTargetRules Target) : base(Target)
	{
		bUseRTTI = true;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Engine",
				"MDLImporter", // Always add the dependency so we don't have to #ifdef our code that uses the MDLImporter includes directly
				"RHI", // Because the MDL shader translator uses GMaxRHIShaderPlatform
				"UnrealUSDWrapper", // Needed to link with USD and its dependencies, like Python
				"USDClasses", // To register the render context
				"USDSchemas", // Because the MDL schema translator actually derives the MDL schema translator
				"USDUtilities", // To register the new schema translator
			}
		);

		UnrealUSDWrapper.CheckAndSetupUsdSdk(Target, this);

		if (Target.bBuildEditor)
		{
			// Note: This path must match whatever the MDLImporter.Build.cs is actually using
			string ExpectedMDLSDKPath = Path.GetFullPath(Path.Combine(EngineDirectory, "Restricted/NotForLicensees/Source/ThirdParty/Enterprise/mdl-sdk-349500.8766a/include"));
			if (Directory.Exists(ExpectedMDLSDKPath))
			{
				// Conditionally adding this define lets us not register the schema translator if the SDK is not usable anyway
				PrivateDefinitions.Add("USE_MDLSDK");
			}
		}
	}
}
