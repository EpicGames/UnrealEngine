{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "USD Importer MDL integration",
	"Description": "Allows importing USD files that reference MDL files, via the USD Stage Actor and USD import",
	"Category": "Importers",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Mac",
		"Linux"
	],
	"Modules": [
		{
			"Name": "USDImporterMDL",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64", "Mac", "Linux"
			]
		}
	],
	"Plugins": [
		{
			"Name": "USDCore",
			"Enabled": true
		},
		{
			"Name": "USDImporter",
			"Enabled": true
		},
		{
			"Name": "MDLImporter",
			"Enabled": true
		}
	]
}